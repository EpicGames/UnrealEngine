// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/SharedString.h"
#include "Misc/ScopeLock.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "HAL/IConsoleManager.h"
#include "ImageCore.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "TextureCompressorModule.h"
#include "PixelFormat.h"
#include "EngineLogs.h"
#include "Async/ParallelFor.h"
#include "TextureBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"
#include "Misc/Paths.h"

THIRD_PARTY_INCLUDES_START
	#include "nvtt/nvtt.h"
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatDXT, Log, All);

class FDXTTextureBuildFunction final : public FTextureBuildFunction
{
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(UTF8TEXTVIEW("DXTTexture"));
		return Name;
	}

	void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const final
	{
		static FGuid Version(TEXT("c2d5dbc5-131c-4525-a332-843230076d99"));
		Builder << Version;
		OutTextureFormatVersioning = FModuleManager::GetModuleChecked<ITextureFormatModule>(TEXT("TextureFormatDXT")).GetTextureFormat();
	}
};

/**
 * Macro trickery for supported format names.
 */
#define ENUM_SUPPORTED_FORMATS(op) \
	op(DXT1) \
	op(DXT3) \
	op(DXT5) \
	op(AutoDXT) \
	op(DXT5n) \
	op(BC4)	\
	op(BC5)

#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
#undef DECL_FORMAT_NAME

#define DECL_FORMAT_NAME_ENTRY(FormatName) GTextureFormatName##FormatName ,
static FName GSupportedTextureFormatNames[] =
{
	ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME_ENTRY)
};
#undef DECL_FORMAT_NAME_ENTRY

#undef ENUM_SUPPORTED_FORMATS

/**
 * NVTT output handler.
 */
struct FNVOutputHandler : public nvtt::OutputHandler
{
	explicit FNVOutputHandler( uint8* InBuffer, int64 InBufferSize )
		: Buffer(InBuffer)
		, BufferEnd(InBuffer + InBufferSize)
	{
	}

	~FNVOutputHandler()
	{
	}

	virtual void beginImage( int size, int width, int height, int depth, int face, int miplevel )
	{
	}

	virtual bool writeData( const void* data, int size )
	{
		check(data);
		check(Buffer + size <= BufferEnd);
		FMemory::Memcpy(Buffer, data, size);
		Buffer += size;
		return true;
	}

    virtual void endImage()
    {
    }

	uint8* Buffer;
	uint8* BufferEnd;
};

/**
 * NVTT error handler.
 */
struct FNVErrorHandler : public nvtt::ErrorHandler
{
	FNVErrorHandler() : 
		bSuccess(true)
	{}

	virtual void error(nvtt::Error e)
	{
		UE_LOGF(LogTextureFormatDXT, Warning, "nvtt::compress() failed with error '%ls'", ANSI_TO_TCHAR(nvtt::errorString(e)));
		bSuccess = false;
	}

	bool bSuccess;
};

/**
 * All state objects needed for NVTT.
 */
class FNVTTCompressor
{
	FNVOutputHandler			OutputHandler;
	FNVErrorHandler				ErrorHandler;
	nvtt::InputOptions			InputOptions;
	nvtt::CompressionOptions	CompressionOptions;
	nvtt::OutputOptions			OutputOptions;
	nvtt::Compressor			Compressor;

public:
	/** Initialization constructor. */
	FNVTTCompressor(
		const void* SourceData,
		EPixelFormat PixelFormat,
		int32 SizeX,
		int32 SizeY,
		bool bSRGB,
		bool bIsNormalMap,
		uint8* OutBuffer,
		int64 BufferSize, 
		bool bPreview = false)
		: OutputHandler(OutBuffer, BufferSize)
	{
		// CUDA acceleration currently disabled, needs more robust error handling
		// With one core of a Xeon 3GHz CPU, compressing a 2048^2 normal map to DXT1 with NVTT 2.0.4 takes 7.49s.
		// With the same settings but using CUDA and a Geforce 8800 GTX it takes 1.66s.
		// To use CUDA, a CUDA 2.0 capable driver is required (178.08 or greater) and a Geforce 8 or higher.
		const bool bUseCUDAAcceleration = false;

		// DXT1a support is currently not exposed.
		const bool bSupportDXT1a = false;

		// Quality level is hardcoded to production quality for now.
		const nvtt::Quality QualityLevel = bPreview ? nvtt::Quality_Fastest : nvtt::Quality_Production;

		nvtt::Format TextureFormat = nvtt::Format_DXT1;
		if (PixelFormat == PF_DXT1)
		{
			TextureFormat = bSupportDXT1a ? nvtt::Format_DXT1a : nvtt::Format_DXT1;
		}
		else if (PixelFormat == PF_DXT3)
		{
			TextureFormat = nvtt::Format_DXT3;
		}
		else if (PixelFormat == PF_DXT5 && bIsNormalMap)
		{
			TextureFormat = nvtt::Format_DXT5n;
		}
		else if (PixelFormat == PF_DXT5)
		{
			TextureFormat = nvtt::Format_DXT5;
		}
		else if (PixelFormat == PF_B8G8R8A8)
		{
			TextureFormat = nvtt::Format_RGBA;
		}
		else if (PixelFormat == PF_BC4)
		{
			TextureFormat = nvtt::Format_BC4;
		}
		else if (PixelFormat == PF_BC5)
		{
			TextureFormat = nvtt::Format_BC5;
		}
		else
		{
			UE_LOGF(LogTextureFormatDXT,Fatal,
				"Unsupported EPixelFormat for compression: %u",
				(uint32)PixelFormat
				);
		}

		InputOptions.setTextureLayout(nvtt::TextureType_2D, SizeX, SizeY);

		// Not generating mips with NVTT, we will pass each mip in and compress it individually
		InputOptions.setMipmapGeneration(false, -1);
		verify(InputOptions.setMipmapData(SourceData, SizeX, SizeY));

		if (bSRGB)
		{
			InputOptions.setGamma(2.2f, 2.2f);
		}
		else
		{
			InputOptions.setGamma(1.0f, 1.0f);
		}

		// Only used for mip and normal map generation
		InputOptions.setWrapMode(nvtt::WrapMode_Mirror);
		InputOptions.setFormat(nvtt::InputFormat_BGRA_8UB);

		// Highest quality is 2x slower with only a small visual difference
		// Might be worthwhile for normal maps though
		CompressionOptions.setQuality(QualityLevel);
		CompressionOptions.setFormat(TextureFormat);

		if ( bIsNormalMap )
		{
			// For BC5 normal maps we don't care about the blue channel.
			CompressionOptions.setColorWeights( 1.0f, 1.0f, 0.0f );

			// Don't tell NVTT it's a normal map. It was producing noticeable artifacts during BC5 compression.
			//InputOptions.setNormalMap(true);
		}
		else
		{
			CompressionOptions.setColorWeights(1, 1, 1);
		}

		Compressor.enableCudaAcceleration(bUseCUDAAcceleration);
		//OutputHandler.ReserveMemory( Compressor.estimateSize(InputOptions, CompressionOptions) );
		check(OutputHandler.BufferEnd - OutputHandler.Buffer <= Compressor.estimateSize(InputOptions, CompressionOptions));

		// We're not outputting a dds file so disable the header
		OutputOptions.setOutputHeader( false );
		OutputOptions.setOutputHandler( &OutputHandler );
		OutputOptions.setErrorHandler( &ErrorHandler );
	}

	/** Run the compressor. */
	bool Compress()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNVTTCompressor::Compress);
		return Compressor.process(InputOptions, CompressionOptions, OutputOptions) && ErrorHandler.bSuccess;
	}
};

/**
 * Asynchronous NVTT worker.
 */
class FAsyncNVTTWorker
{
public:
	/**
	 * Initializes the data and creates the async compression task.
	 */
	FAsyncNVTTWorker(FNVTTCompressor* InCompressor)
		: Compressor(InCompressor)
	{
		check(Compressor);
	}

	/** Compresses the texture. */
	void DoWork()
	{
		bCompressionResults = Compressor->Compress();
	}

	/** Retrieve compression results. */
	bool GetCompressionResults() const { return bCompressionResults; }

private:
	/** The NVTT compressor. */
	FNVTTCompressor* Compressor;
	/** true if compression was successful. */
	bool bCompressionResults;
};

namespace CompressionSettings
{
	int32 BlocksPerBatch = 2048;
	FAutoConsoleVariableRef BlocksPerBatch_CVar(
		TEXT("Tex.AsyncDXTBlocksPerBatch"),
		BlocksPerBatch,
		TEXT("The number of blocks to compress in parallel for DXT compression.")
		);
}

/**
 * Compresses an image using NVTT.
 * @param SourceData			Source texture data to DXT compress, in BGRA 8bit per channel unsigned format.
 * @param PixelFormat			Texture format
 * @param SizeX					Number of texels along the X-axis
 * @param SizeY					Number of texels along the Y-axis
 * @param bSRGB					Whether the texture is in SRGB space
 * @param bIsNormalMap			Whether the texture is a normal map
 * @param OutCompressedData		Compressed image data output by nvtt.
 */
static bool CompressImageUsingNVTT(
	const void* SourceData,
	EPixelFormat PixelFormat,
	int32 SizeX,
	int32 SizeY,
	bool bSRGB,
	bool bIsNormalMap,
	bool bIsPreview,
	TArray64<uint8>& OutCompressedData
	)
{
	check(PixelFormat == PF_DXT1 || PixelFormat == PF_DXT3 || PixelFormat == PF_DXT5 || PixelFormat == PF_BC4 || PixelFormat == PF_BC5);

	// Avoid dependency on GPixelFormats in RenderCore.
	const int32 BlockSizeX = 4;
	const int32 BlockSizeY = 4;
	const int32 BlockBytes = (PixelFormat == PF_DXT1 || PixelFormat == PF_BC4) ? 8 : 16;
	const int32 ImageBlocksX = FMath::Max( FMath::DivideAndRoundUp( SizeX , BlockSizeX), 1);
	const int32 ImageBlocksY = FMath::Max( FMath::DivideAndRoundUp( SizeY , BlockSizeY), 1);
	const int32 BlocksPerBatch = FMath::Max<int32>(ImageBlocksX, FMath::RoundUpToPowerOfTwo(CompressionSettings::BlocksPerBatch));
	const int32 RowsPerBatch = BlocksPerBatch / ImageBlocksX;
	const int32 NumBatches = ImageBlocksY / RowsPerBatch;
	// these round down, then if (RowsPerBatch * NumBatches) != ImageBlocksY , will encode without batches

	// nvtt doesn't support 64-bit output sizes.
	int64 OutDataSize = (int64)ImageBlocksX * ImageBlocksY * BlockBytes;
	if (OutDataSize > MAX_uint32)
	{
		return false;
	}

	// Allocate space to store compressed data.
	OutCompressedData.Empty(OutDataSize);
	OutCompressedData.AddUninitialized(OutDataSize);

	if (ImageBlocksX * ImageBlocksY <= BlocksPerBatch ||
		BlocksPerBatch % ImageBlocksX != 0 ||
		RowsPerBatch * NumBatches != ImageBlocksY)
	{
		FNVTTCompressor* Compressor = NULL;
		{
			Compressor = new FNVTTCompressor(
				SourceData,
				PixelFormat,
				SizeX,
				SizeY,
				bSRGB,
				bIsNormalMap,
				OutCompressedData.GetData(),
				OutCompressedData.Num(),
				bIsPreview
				);
		}
		bool bSuccess = Compressor->Compress();
		{
			delete Compressor;
			Compressor = NULL;
		}
		return bSuccess;
	}

	int64 UncompressedStride = (int64)RowsPerBatch * BlockSizeY * SizeX * sizeof(FColor);
	int32 CompressedStride = RowsPerBatch * ImageBlocksX * BlockBytes;

	// Create compressors for each batch.
	TIndirectArray<FNVTTCompressor> Compressors;
	Compressors.Empty(NumBatches);
	{
		const uint8* Src = (const uint8*)SourceData;
		uint8* Dest = OutCompressedData.GetData();
		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			Compressors.Add(new FNVTTCompressor(
				Src,
				PixelFormat,
				SizeX,
				RowsPerBatch * BlockSizeY,
				bSRGB,
				bIsNormalMap,
				Dest,
				CompressedStride
			));
			Src += UncompressedStride;
			Dest += CompressedStride;
		}
	}

	// Asynchronously compress each batch.
	bool bSuccess = true;
	{
		TArray<FAsyncNVTTWorker> AsyncTasks;
		AsyncTasks.Reserve(NumBatches);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			AsyncTasks.Emplace(&Compressors[BatchIndex]);
		}

		ParallelForTemplate(AsyncTasks.Num(), [&AsyncTasks](int32 TaskIndex)
		{
			AsyncTasks[TaskIndex].DoWork();
		}, EParallelForFlags::Unbalanced);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			bSuccess = bSuccess && AsyncTasks[BatchIndex].GetCompressionResults();
		}
	}

	// Release compressors
	{
		Compressors.Empty();
	}

	return bSuccess;
}

/**
 * DXT texture format handler.
 */
class FTextureFormatDXT : public ITextureFormat
{
public:
	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual FName GetEncoderName(FName Format) const override
	{
		static const FName DXTName("EngineDXT");
		return DXTName;
	}

	virtual uint16 GetVersion(
		FName Format,
		const struct FTextureBuildSettings* BuildSettings = nullptr
	) const override
	{
		return 0;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(GSupportedTextureFormatNames); ++i)
		{
			OutFormats.Add(GSupportedTextureFormatNames[i]);
		}
	}
		
	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& BuildSettings, bool bImageHasAlphaChannel) const override
	{
		if (BuildSettings.TextureFormatName == GTextureFormatNameDXT1)
		{
			return PF_DXT1;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameDXT3)
		{
			return PF_DXT3;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameDXT5)
		{
			return PF_DXT5;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameAutoDXT)
		{
			return bImageHasAlphaChannel ? PF_DXT5 : PF_DXT1;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameDXT5n)
		{
			return PF_DXT5;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameBC5)
		{
			return PF_BC5;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameBC4)
		{
			return PF_BC4;
		}

		UE_LOGF(LogTextureFormatDXT, Fatal, "Unhandled texture format '%ls' given to FTextureFormatDXT::GetEncodedPixelFormat()", *BuildSettings.TextureFormatName.ToString());
		return PF_Unknown;
	}

	virtual bool CompressImage(
		const FImage& InImage,
		const struct FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		FCompressedImage2D& OutCompressedImage
		) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTextureFormatDXT::CompressImage);
		
		// now we know NVTT will actually be used, Load the DLL :
		const_cast<FTextureFormatDXT *>(this)->LoadDLL();

		FImage Image;
		InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

		EPixelFormat CompressedPixelFormat = GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel);
		bool bIsNormalMap = BuildSettings.TextureFormatName == GTextureFormatNameDXT5n || BuildSettings.TextureFormatName == GTextureFormatNameBC5;

		bool bCompressionSucceeded = true;
		int64 SliceSize = (int64)Image.SizeX * Image.SizeY;

		if (Image.NumSlices == 1 && OutCompressedImage.RawData.Num() == 0)
		{
			// Avoid using a temp buffer when it's not needed
			bCompressionSucceeded = CompressImageUsingNVTT(
				(&Image.AsBGRA8()[0]),
				CompressedPixelFormat,
				Image.SizeX,
				Image.SizeY,
				Image.IsGammaCorrected(),
				bIsNormalMap,
				false, // Daniel Lamb: Testing with this set to true didn't give large performance gain to lightmaps.  Encoding of 140 lightmaps was 19.2seconds with preview 20.1 without preview.  11/30/2015
				OutCompressedImage.RawData
				);
		}
		else
		{
			for (int32 SliceIndex = 0; SliceIndex < Image.NumSlices && bCompressionSucceeded; ++SliceIndex)
			{
				TArray64<uint8> CompressedSliceData;
				bCompressionSucceeded = CompressImageUsingNVTT(
					(&Image.AsBGRA8()[0]) + SliceIndex * SliceSize,
					CompressedPixelFormat,
					Image.SizeX,
					Image.SizeY,
					Image.IsGammaCorrected(),
					bIsNormalMap,
					false, // Daniel Lamb: Testing with this set to true didn't give large performance gain to lightmaps.  Encoding of 140 lightmaps was 19.2seconds with preview 20.1 without preview.  11/30/2015
					CompressedSliceData
					);
				OutCompressedImage.RawData.Append(MoveTemp(CompressedSliceData));
			}
		}

		if (bCompressionSucceeded)
		{
			// no more image size padding here
			OutCompressedImage.SizeX = Image.SizeX;
			OutCompressedImage.SizeY = Image.SizeY;
			// old behavior :
			//OutCompressedImage.SizeX = FMath::Max(Image.SizeX, 4);
			//OutCompressedImage.SizeY = FMath::Max(Image.SizeY, 4);
			OutCompressedImage.NumSlicesWithDepth = Image.NumSlices;
			OutCompressedImage.PixelFormat = CompressedPixelFormat;
		}
		return bCompressionSucceeded;
	}

	FTextureFormatDXT()
	{
		// don't LoadDLL until this format is actually used
	}

	void LoadDLL()
	{
#if PLATFORM_WINDOWS
		// nvtt_64.dll is set to DelayLoad by nvTextureTools.Build.cs
		// manually load before any call to it, because it's not put in the binaries search path,
		// and so we can get the AVX2 variant or not :

		if ( nvTextureToolsHandle != nullptr )
		{
			return;
		}

		// Lock so only one thread does init :
		FScopeLock HandleLock(&nvTextureToolsHandleLock);
		
		// double check inside lock :
		if ( nvTextureToolsHandle != nullptr )
		{
			return;
		}
#if PLATFORM_CPU_ARM_FAMILY
		nvTextureToolsHandle = FPlatformProcess::GetDllHandle(*(FPaths::EngineDir() / TEXT("Binaries/ThirdParty/nvTextureTools/Win64/arm64/nvtt_64.dll")));
#else
		if (FWindowsPlatformMisc::HasAVX2InstructionSupport())
		{
			nvTextureToolsHandle = FPlatformProcess::GetDllHandle(*(FPaths::EngineDir() / TEXT("Binaries/ThirdParty/nvTextureTools/Win64/AVX2/nvtt_64.dll")));
		}
		else
		{
			nvTextureToolsHandle = FPlatformProcess::GetDllHandle(*(FPaths::EngineDir() / TEXT("Binaries/ThirdParty/nvTextureTools/Win64/nvtt_64.dll")));
		}
#endif  //PLATFORM_CPU_ARM_FAMILY
#endif	//PLATFORM_WINDOWS
	}

	~FTextureFormatDXT()
	{
#if PLATFORM_WINDOWS
		if ( nvTextureToolsHandle != nullptr )
		{
			FPlatformProcess::FreeDllHandle(nvTextureToolsHandle);
			nvTextureToolsHandle = nullptr;
		}
#endif
	}
	
#if PLATFORM_WINDOWS
	// Handle to the nvtt dll
	void* nvTextureToolsHandle = nullptr;
	FCriticalSection nvTextureToolsHandleLock;
#endif	//PLATFORM_WINDOWS
};

/**
 * Module for DXT texture compression.
 */
static ITextureFormat* Singleton = NULL;

class FTextureFormatDXTModule : public ITextureFormatModule
{
public:
	virtual ~FTextureFormatDXTModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	
	virtual bool CanCallGetTextureFormats() override { return false; }

	virtual ITextureFormat* GetTextureFormat()
	{
		if (!Singleton)
		{
			Singleton = new FTextureFormatDXT();
		}
		return Singleton;
	}

	// IModuleInterface implementation.
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	static inline UE::DerivedData::TBuildFunctionFactory<FDXTTextureBuildFunction> BuildFunctionFactory;

private:
};

IMPLEMENT_MODULE(FTextureFormatDXTModule, TextureFormatDXT);
