// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormatDXT : ModuleRules
{
	public TextureFormatDXT(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"DerivedDataCache",
			"Engine",
			"TextureCompressor",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"ImageCore",
			"ImageWrapper",
			"TextureBuild",
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "nvTextureTools");
	}
}
